[/Script/UnrealEd.ProjectPackagingSettings]
+DirectoriesToAlwaysCook=(Path="/Engine/VREditor/Devices/Generic")
+DirectoriesToAlwaysCook=(Path="/Engine/VREditor/Devices/Oculus")
+DirectoriesToAlwaysCook=(Path="/Engine/VREditor/Devices/Vive")
+DirectoriesToAlwaysCook=(Path="/Engine/VREditor/Devices/STEM")
