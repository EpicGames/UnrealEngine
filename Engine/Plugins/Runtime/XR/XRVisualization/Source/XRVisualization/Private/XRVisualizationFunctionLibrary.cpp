// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRVisualizationFunctionLibrary.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Features/IModularFeatures.h"
#include "HeadMountedDisplayTypes.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Package.h"
#include "XRVisualizationModule.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "TimerManager.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/ConstructorHelpers.h"
#include "ProceduralMeshComponent.h"
#include "IHandTracker.h"


UXRVisualizationLoadHelper::UXRVisualizationLoadHelper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsDefaultSubobject())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UStaticMesh> GenericHMDMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> OculusControllerMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> ViveControllerMesh;
			ConstructorHelpers::FObjectFinder<UStaticMesh> STEMControllerMesh;
			FConstructorStatics()
				: GenericHMDMesh(TEXT("/Engine/VREditor/Devices/Generic/GenericHMD.GenericHMD"))
				, OculusControllerMesh(TEXT("/Engine/VREditor/Devices/Oculus/OculusControllerMesh.OculusControllerMesh"))
				, ViveControllerMesh(TEXT("/Engine/VREditor/Devices/Vive/ViveControllerMesh.ViveControllerMesh"))
				, STEMControllerMesh(TEXT("/Engine/VREditor/Devices/STEM/STEMControllerMesh.STEMControllerMesh"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GenericHMD = ConstructorStatics.GenericHMDMesh.Object;
		OculusControllerMesh = ConstructorStatics.OculusControllerMesh.Object;
		ViveControllerMesh = ConstructorStatics.ViveControllerMesh.Object;
		STEMControllerMesh = ConstructorStatics.STEMControllerMesh.Object;

		if ((GenericHMD == nullptr) || (OculusControllerMesh == nullptr) || (ViveControllerMesh == nullptr) || (STEMControllerMesh == nullptr))
		{
			UE_LOGF(LogXRVisual, Error, "XR Visualizations failed to load");
		}
	}
}

UXRVisualizationFunctionLibrary::~UXRVisualizationFunctionLibrary()
{
	if (LoadHelper)
	{
		LoadHelper->RemoveFromRoot();
		LoadHelper = nullptr;
	}
}

void UXRVisualizationFunctionLibrary::RenderHMD(const FXRHMDData& XRHMDData)
{
	UXRVisualizationFunctionLibrary* Instance = UXRVisualizationFunctionLibrary::StaticClass()->GetDefaultObject<UXRVisualizationFunctionLibrary>();
	check(Instance);
	Instance->VerifyInitMeshes();
	UXRVisualizationLoadHelper* LocalLoadHelper = Instance->LoadHelper;
	check(LocalLoadHelper);

	UStaticMesh* MeshToUse = LocalLoadHelper->GenericHMD;
	FTransform WorldTransform(XRHMDData.Rotation, XRHMDData.Position);

	FName ActorName(*FString::Printf(TEXT("XR_%s_%llx"), *XRHMDData.DeviceName.ToString(), (PTRINT)&XRHMDData));

	//now render at the proper transform
	if (MeshToUse != nullptr)
	{
		RenderGenericMesh(ActorName, MeshToUse, WorldTransform);
	}
}

void UXRVisualizationFunctionLibrary::RenderMotionController2(const FXRMotionControllerState& XRControllerState)
{
	if (XRControllerState.bValid)
	{
		//now render at the proper transform
		UXRVisualizationFunctionLibrary* Instance = UXRVisualizationFunctionLibrary::StaticClass()->GetDefaultObject<UXRVisualizationFunctionLibrary>();
		check(Instance);
		Instance->VerifyInitMeshes();
		UXRVisualizationLoadHelper* LocalLoadHelper = Instance->LoadHelper;
		check(LocalLoadHelper);
		
		const bool bRight = XRControllerState.Hand == EControllerHand::Right;
		FTransform WorldTransform(XRControllerState.GripUnrealSpaceRotation, XRControllerState.GripUnrealSpaceLocation, bRight ? FVector(1.0f, -1.0f, 1.0f) : FVector(1.0f, 1.0f, 1.0f));

		//take the least significant bits of the pointer to use as a unique identifier
		uint32 PtrInt = (((PTRINT)&XRControllerState) & 0xffffff);
		int32 HandIndexInt = (int)XRControllerState.Hand;
		FName ActorName(*FString::Printf(TEXT("XR_%s_%x_%d"), *XRControllerState.DeviceName.ToString(), PtrInt, HandIndexInt));

		UStaticMesh* MeshToUse = nullptr;
		if (XRControllerState.DeviceName == TEXT("OculusHMD"))
		{
			MeshToUse = LocalLoadHelper->OculusControllerMesh;
		}
		else if (XRControllerState.DeviceName == TEXT("SteamVR"))
		{
			MeshToUse = LocalLoadHelper->ViveControllerMesh;
		}
		else //if (XRVisualizationData.DeviceName == TEXT(""))
		{
			MeshToUse = LocalLoadHelper->STEMControllerMesh;
		}

		if (MeshToUse != nullptr)
		{
			RenderGenericMesh(ActorName, MeshToUse, WorldTransform);
		}
	}
}

void UXRVisualizationFunctionLibrary::RenderHandTracking(const FXRHandTrackingState& XRHandTrackingState)
{
	bool bLocalRender = false;

	if (XRHandTrackingState.ApplicationInstanceID == FApp::GetInstanceId())
	{
		TArray<IHandTracker*> HandTrackers = IModularFeatures::Get().GetModularFeatureImplementations<IHandTracker>(IHandTracker::GetModularFeatureName());
		IHandTracker* HandTracker = nullptr;
		for (auto Itr : HandTrackers)
		{
			if (Itr->IsHandTrackingStateValid() && Itr->HasHandMeshData())
			{
				HandTracker = Itr;
				break;
			}
		}

		if (HandTracker != nullptr)
		{
			int32 HandIndex = (int)XRHandTrackingState.Hand;
			if ((HandIndex < 0) || (HandIndex >= 2))
			{
				return;
			}

			TArray<FVector> Vertices, Normals;
			TArray<int32> Indices;
			FTransform HandMeshTransform;

			if ((HandTracker->GetHandMeshData((EControllerHand)HandIndex, Vertices, Normals, Indices, HandMeshTransform)) && (Vertices.Num() > 0))
			{
				bLocalRender = true;

				FName ActorName(*FString::Printf(TEXT("XR_hand_mesh_%s_%llx_%d"), *XRHandTrackingState.DeviceName.ToString(), (PTRINT)&XRHandTrackingState, HandIndex));
				ULevel* CurrentLevel = GWorld->GetCurrentLevel();

				AActor* FoundActor = FindObjectFast<AActor>(CurrentLevel, ActorName, EFindObjectFlags::ExactClass);
				UProceduralMeshComponent* ProceduralMeshComponent = nullptr;
				TArray<FLinearColor> Colors(&FLinearColor::White, Vertices.Num());

				if (!IsValid(FoundActor))
				{
					//create actor
					FoundActor = NewObject<AActor>(CurrentLevel, ActorName);
					//create component
					ProceduralMeshComponent = NewObject<UProceduralMeshComponent>(FoundActor, UProceduralMeshComponent::StaticClass());
					ProceduralMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
					ProceduralMeshComponent->RegisterComponentWithWorld(GWorld);

					ProceduralMeshComponent->CreateMeshSection_LinearColor(0, Vertices, Indices, Normals, {}, Colors, {}, false);

					// RobM: I couldn't find this section in any of the ini files, does it still exist? if so can HandMeshMaterial be converted to a path name?
					if (GConfig->DoesSectionExist(TEXT("/Script/EngineSettings.XRVisualizationSettings"), GGameIni))
					{
						FString MaterialAddress;
						GConfig->GetString(TEXT("/Script/EngineSettings.XRVisualizationSettings"), TEXT("HandMeshMaterial"),
							MaterialAddress, GGameIni);

						if (!MaterialAddress.IsEmpty())
						{
							UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(StaticFindFirstObject(UMaterialInterface::StaticClass(), *MaterialAddress));
							ProceduralMeshComponent->SetMaterial(0, MaterialInterface);
						}
					}
				}
				else
				{
					//get static mesh component out of the actor
					ProceduralMeshComponent = Cast<UProceduralMeshComponent>(FoundActor->GetComponentByClass(UProceduralMeshComponent::StaticClass()));

					if (ProceduralMeshComponent)
					{
						ProceduralMeshComponent->UpdateMeshSection_LinearColor(0, Vertices, Normals, {}, Colors, {});
					}
				}

				if (ProceduralMeshComponent)
				{
					//Update the transform
					ProceduralMeshComponent->SetWorldTransform(HandMeshTransform);

					FTimerManager& TimerManager = FoundActor->GetWorldTimerManager();
					//reset this timer and start over
					TimerManager.ClearAllTimersForObject(FoundActor);

					//Add timer to turn the component back off
					FTimerHandle TempHandle;
					TimerManager.SetTimer(TempHandle, FoundActor, &AActor::K2_DestroyActor, 2.0f, true);
				}
			}
		}
	}

	//in case no hand tracker provided rendering 
	if (!bLocalRender)
	{
		RenderFinger(XRHandTrackingState, (int32)EHandKeypoint::ThumbMetacarpal, (int32)EHandKeypoint::ThumbTip);
		RenderFinger(XRHandTrackingState, (int32)EHandKeypoint::IndexMetacarpal, (int32)EHandKeypoint::IndexTip);
		RenderFinger(XRHandTrackingState, (int32)EHandKeypoint::MiddleMetacarpal, (int32)EHandKeypoint::MiddleTip);
		RenderFinger(XRHandTrackingState, (int32)EHandKeypoint::RingMetacarpal, (int32)EHandKeypoint::RingTip);
		RenderFinger(XRHandTrackingState, (int32)EHandKeypoint::LittleMetacarpal, (int32)EHandKeypoint::LittleTip);
	}
}

void UXRVisualizationFunctionLibrary::RenderFinger(const FXRHandTrackingState& XRHandTrackingState, int32 FingerStart, int32 FingerEnd)
{
	check((FingerStart > 0) && (FingerStart < EHandKeypointCount));
	check((FingerEnd > 0) && (FingerEnd < EHandKeypointCount));
	if ((EHandKeypointCount == XRHandTrackingState.HandKeyLocations.Num()) && (EHandKeypointCount == XRHandTrackingState.HandKeyRadii.Num()))
	{
		FColor Color = FColor::Magenta;
		bool bPersistentLines = false;
		float LifeTime = -1.0f;
		uint8 DepthPriority = 0;
		float Thickness = 0.05f;

		//draw 0 - 1 (palm to wrist)
		DrawDebugLine(GWorld, XRHandTrackingState.HandKeyLocations[0], XRHandTrackingState.HandKeyLocations[1], Color, bPersistentLines, LifeTime, DepthPriority, Thickness);

		//draw 1 - LineStart (wrist to first part of finger)
		DrawDebugLine(GWorld, XRHandTrackingState.HandKeyLocations[1], XRHandTrackingState.HandKeyLocations[FingerStart], Color, bPersistentLines, LifeTime, DepthPriority, Thickness);

		//Iterate from FingerStart to Finger End
		for (int DigitIndex = FingerStart; DigitIndex < FingerEnd; ++DigitIndex)
		{
			DrawDebugLine(GWorld, XRHandTrackingState.HandKeyLocations[DigitIndex], XRHandTrackingState.HandKeyLocations[DigitIndex + 1], Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
			//FVector Extents(XRHandTrackingState.HandKeyRadii[DigitIndex]);
			//DrawDebugBox(GWorld, XRHandTrackingState.HandKeyLocations[DigitIndex], Extents, FQuat::Identity, RadiusColor, bPersistentLines, LifeTime, DepthPriority, Thickness);
			DrawDebugSphere(GWorld, XRHandTrackingState.HandKeyLocations[DigitIndex + 1], XRHandTrackingState.HandKeyRadii[DigitIndex + 1], 4, Color, bPersistentLines, LifeTime, DepthPriority, Thickness);
		}
	}
}

void UXRVisualizationFunctionLibrary::VerifyInitMeshes()
{
	if (LoadHelper == nullptr)
	{
		LoadHelper = NewObject<UXRVisualizationLoadHelper>(GetTransientPackage(), UXRVisualizationLoadHelper::StaticClass());
		LoadHelper->AddToRoot();
	}
}

void UXRVisualizationFunctionLibrary::RenderGenericMesh(const FName& ActorName, UStaticMesh* Mesh, FTransform& WorldTransform)
{
	ULevel* CurrentLevel = GWorld->GetCurrentLevel();

	AActor* FoundActor = FindObjectFast<AActor>(CurrentLevel, ActorName, EFindObjectFlags::ExactClass);
	UStaticMeshComponent* StaticMeshComponent = nullptr;
	if (!IsValid(FoundActor))
	{
		//create actor
		FoundActor = NewObject<AActor>(CurrentLevel, ActorName);
		//create component
		StaticMeshComponent = NewObject<UStaticMeshComponent>(FoundActor, UStaticMeshComponent::StaticClass());
		StaticMeshComponent->SetStaticMesh(Mesh);
		StaticMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		StaticMeshComponent->RegisterComponentWithWorld(GWorld);
	}
	else
	{
		//get static mesh component out of the actor
		StaticMeshComponent = Cast<UStaticMeshComponent>(FoundActor->GetComponentByClass(UStaticMeshComponent::StaticClass()));
	}

	if (StaticMeshComponent)
	{
		//Update the transform
		StaticMeshComponent->SetWorldTransform(WorldTransform);

		FTimerManager& TimerManager = FoundActor->GetWorldTimerManager();
		//reset this timer and start over
		TimerManager.ClearAllTimersForObject(FoundActor);	

		//Add timer to turn the component back off
		FTimerHandle TempHandle;
		TimerManager.SetTimer(TempHandle, FoundActor, &AActor::K2_DestroyActor, 2.0f, true);
	}

}
