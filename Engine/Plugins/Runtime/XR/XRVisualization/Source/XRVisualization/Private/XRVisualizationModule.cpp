// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRVisualizationModule.h"
#include "Modules/ModuleManager.h"

class FXRVisualizationModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FXRVisualizationModule, XRVisualization);

DEFINE_LOG_CATEGORY(LogXRVisual);
