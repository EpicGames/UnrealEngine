// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "HeadMountedDisplayTypes.h"
#include "XRVisualizationFunctionLibrary.generated.h"

UCLASS()
class UXRVisualizationLoadHelper : public UObject
{
	GENERATED_BODY()

	/** UObject constructor that takes an optional ObjectInitializer */
	UXRVisualizationLoadHelper(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

public:
	UPROPERTY()
	TObjectPtr<class UStaticMesh> GenericHMD;
	UPROPERTY()
	TObjectPtr<class UStaticMesh> OculusControllerMesh;
	UPROPERTY()
	TObjectPtr<class UStaticMesh> ViveControllerMesh;
	UPROPERTY()
	TObjectPtr<class UStaticMesh> STEMControllerMesh;
};

UCLASS()
class XRVISUALIZATION_API UXRVisualizationFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	virtual ~UXRVisualizationFunctionLibrary();

	// These methods should be visible from C++.
public:
	/**
	 * Renders HMD Visualization on a system that might not have that HMD as native
	 */
	UFUNCTION(BlueprintCallable, Category = "Input|XRTracking")
	static void RenderHMD(const FXRHMDData& XRHMDData);

	/**
	 * Renders Motion Controller Visualization on a system that might not have that HMD as native
	 */
	UFUNCTION(BlueprintCallable, Category = "Input|XRTracking")
	static void RenderMotionController2(const FXRMotionControllerState& XRControllerState);
	/**
	 * Renders Motion Controller Visualization on a system that might not have that HMD as native
	 */
	UFUNCTION(BlueprintCallable, Category = "Input|XRTracking")
	static void RenderHandTracking(const FXRHandTrackingState& XRHandTrackingState);

private:
	void VerifyInitMeshes();

	static void RenderGenericMesh(const FName& ActorName, UStaticMesh* Mesh, FTransform& WorldTransform);

	static void RenderFinger(const FXRHandTrackingState& XRHandTrackingState, int32 FingerStart, int32 FingerEnd);

	UPROPERTY()
	TObjectPtr<UXRVisualizationLoadHelper> LoadHelper;
};
