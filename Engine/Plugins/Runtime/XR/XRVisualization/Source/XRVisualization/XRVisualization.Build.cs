// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class XRVisualization : ModuleRules
	{
		public XRVisualization(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
                    "EngineSettings",
                    "RenderCore",
					"RHI",
					"HeadMountedDisplay",
					"XRBase",
					"ProceduralMeshComponent",
				}
			);
        }
	}
}
