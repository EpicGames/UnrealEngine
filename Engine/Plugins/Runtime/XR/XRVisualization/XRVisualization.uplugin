{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "XRVisualization",
	"Description": "Visualization Library for XR HMDs and controllers",
	"Category": "Mixed Reality",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "XRVisualization",
			"Type": "Runtime"
		}
	],
	"Plugins": [
		{
			"Name": "XRBase",
			"Enabled": true
		},
		{
			"Name": "ProceduralMeshComponent",
			"Enabled": true
		}
	]
}
